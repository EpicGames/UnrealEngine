// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamingPauseRendering.h"
#include "Layout/Margin.h"
#include "ViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "RenderingThread.h"
#include "Modules/ModuleManager.h"
#include "EngineGlobals.h"
#include "CanvasTypes.h"
#include "DummyViewport.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"
#include "Slate/SceneViewport.h"
#include "Slate/SlateTextures.h"
#include "MoviePlayer.h"
#include "Widgets/Images/SThrobber.h"


IMPLEMENT_MODULE(FStreamingPauseRenderingModule, StreamingPauseRendering);

class FBackgroundView : public ISlateViewport
{
public:
	FBackgroundView(FTextureRHIRef RenderTargetTexture, FIntPoint InSize)
		: RenderTarget( new FSlateRenderTargetRHI(RenderTargetTexture, InSize.X, InSize.Y))
		, Size(InSize)
	{
		BeginInitResource(RenderTarget);
	}

	~FBackgroundView()
	{
		ReleaseResourceAndFlush(RenderTarget);
		delete RenderTarget;
	}

	/* ISlateViewport interface. */
	virtual FIntPoint GetSize() const override
	{
		return Size;
	}

	virtual FSlateShaderResource* GetViewportRenderTargetTexture() const override
	{
		return RenderTarget;
	}

	virtual bool RequiresVsync() const override
	{
		return false;
	}

private:
	FSlateRenderTargetRHI* RenderTarget;
	FIntPoint Size;
};

static TAutoConsoleVariable<int32> CVarRenderLastFrameInStreamingPause(
	TEXT("r.RenderLastFrameInStreamingPause"),
	1,
	TEXT("If 1 the previous frame is displayed during streaming pause. If zero the screen is left black."),
	ECVF_RenderThreadSafe);


FStreamingPauseRenderingModule::FStreamingPauseRenderingModule()
	: SceneViewport(nullptr)
	, ViewportWidget(nullptr)
	, BackgroundView(nullptr)
	, bMovieWasStarted(false)
{ }


void FStreamingPauseRenderingModule::StartupModule()
{
	BeginDelegate.BindRaw(this, &FStreamingPauseRenderingModule::BeginStreamingPause);
	EndDelegate.BindRaw(this, &FStreamingPauseRenderingModule::EndStreamingPause);
	check(GEngine);
	GEngine->RegisterBeginStreamingPauseRenderingDelegate(&BeginDelegate);
	GEngine->RegisterEndStreamingPauseRenderingDelegate(&EndDelegate);
}


void FStreamingPauseRenderingModule::ShutdownModule()
{
	BeginDelegate.Unbind();
	EndDelegate.Unbind();
	if( GEngine )
	{
		GEngine->RegisterBeginStreamingPauseRenderingDelegate(nullptr);
		GEngine->RegisterEndStreamingPauseRenderingDelegate(nullptr);
	}
}


void FStreamingPauseRenderingModule::BeginStreamingPause( FViewport* GameViewport )
{
	// If a movie is already playing don't bother starting one
	if (IsMoviePlayerEnabled() && (GetMoviePlayer()->IsInitialized() && !GetMoviePlayer()->IsMovieCurrentlyPlaying()))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStreamingPauseRenderingModule::BeginStreamingPause);

		check(GameViewport);

		//Create the viewport widget and add a throbber.
		ViewportWidget = SNew(SViewport)
			.EnableGammaCorrection(false);

		ViewportWidget->SetContent
		(
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			.Padding(FMargin(10.0f))
			[
				SNew(SThrobber)
			]
		);

		//Render the current scene to a new viewport.
		FIntPoint SizeXY = GameViewport->GetSizeXY();
		bool bRenderLastFrame = CVarRenderLastFrameInStreamingPause.GetValueOnGameThread() != 0;
		bRenderLastFrame = bRenderLastFrame && !(GEngine->StereoRenderingDevice.IsValid() && GameViewport->IsStereoRenderingAllowed());

		if(SizeXY.X > 0 && SizeXY.Y > 0 && bRenderLastFrame)
		{
			const TStrongPtrVariant<FViewportClient>& ViewportClientVariant = GameViewport->GetClientPtr();
			FViewportClient* const ViewportClient = ViewportClientVariant.Get();

			SceneViewport = MakeShared<FSceneViewport>(ViewportWidget);

			SceneViewport->UpdateViewportRHI(false, SizeXY.X, SizeXY.Y, EWindowMode::Fullscreen, PF_Unknown);

			SceneViewport->EnqueueBeginRenderFrame(false);

			FCanvas Canvas(SceneViewport.Get(), nullptr, ViewportClient->GetWorld(), ViewportClient->GetWorld()->GetFeatureLevel());
			{
				ViewportClient->Draw(SceneViewport.Get(), &Canvas);
			}
			Canvas.Flush_GameThread();

			SceneViewport->EnqueueEndRenderFrame(false, false);

			BackgroundView = MakeShareable(new FBackgroundView(SceneViewport->GetRenderTargetTexture(), SizeXY));
			ViewportWidget->SetViewportInterface(BackgroundView.ToSharedRef());

		}


		//Create the loading screen and begin rendering the widget.
		FLoadingScreenAttributes LoadingScreen;
		LoadingScreen.bAutoCompleteWhenLoadingCompletes = true;
		LoadingScreen.WidgetLoadingScreen = ViewportWidget; // SViewport from above
		GetMoviePlayer()->SetupLoadingScreen(LoadingScreen);
		GetMoviePlayer()->PlayMovie();

		// We started playing a movie
		bMovieWasStarted = true;
	}
}

void FStreamingPauseRenderingModule::EndStreamingPause()
{
	// Only wait for the movie to finish if we were the one to start it.
	if(bMovieWasStarted)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FStreamingPauseRenderingModule::EndStreamingPause);

		//Stop rendering the loading screen and resume
		GetMoviePlayer()->WaitForMovieToFinish();

		ViewportWidget = nullptr;
		SceneViewport = nullptr;
		BackgroundView = nullptr;

		FlushRenderingCommands();

		bMovieWasStarted = false;
	}
}
