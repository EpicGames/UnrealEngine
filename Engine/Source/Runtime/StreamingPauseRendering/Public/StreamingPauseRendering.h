// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Engine/Engine.h"

class FBackgroundView;
class FSceneViewport;
class SViewport;

/** 
 * Module handling default behavior for streaming pause rendering. 
 * Games can override by calling RegisterBegin/EndStreamingPauseDelegate with their own delegates.
 */
class FStreamingPauseRenderingModule
	: public IModuleInterface
{
public:	

	/** Default constructor. */
	FStreamingPauseRenderingModule();

	/**
	 * Enqueue the streaming pause to suspend rendering during blocking load.
	 */
	virtual void BeginStreamingPause( class FViewport* Viewport );

	/**
	 * Enqueue the streaming pause to resume rendering after blocking load is completed.
	 */
	virtual void EndStreamingPause();

public:

	// IModuleInterface interface

	virtual void StartupModule();	
	virtual void ShutdownModule();

public:

	/** Viewport being used to render the scene once to a target while paused */
	TSharedPtr<FSceneViewport> SceneViewport;
	/** Slate viewport widget used to draw the target */
	TSharedPtr<SViewport> ViewportWidget;
	/** Helper class to translate the RHI render target to Slate */
	TSharedPtr<class FBackgroundView> BackgroundView;

	/** Delegate providing default functionality for beginning streaming pause. */
	FBeginStreamingPauseDelegate BeginDelegate;

	/** Delegate providing default functionality for ending streaming pause. */
	FEndStreamingPauseDelegate EndDelegate;

	/**
	 * If a movie was started by BeginStreamingPause.  
	 * This could be false if a movie was already playing when BeginStreamingPause was called
	 */
	bool bMovieWasStarted;
};
