// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StreamingPauseRendering : ModuleRules
{
    public StreamingPauseRendering(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
					"Engine",
                    "MoviePlayer",
                    "Slate",
				}
		);

		PrivateDependencyModuleNames.AddRange(
            new string[] {
                    "Core",
                    "InputCore",
                    "RenderCore",
                    "CoreUObject",
                    "RHI",
					"SlateCore",
				}
        );
	}
}
